// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowser.h"
#include "SWebBrowser.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "WebBrowserModule.h"

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "WebBrowser"

/////////////////////////////////////////////////////
// UWebBrowser

UWebBrowser::UWebBrowser(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = true;
	
	// we keep a hard reference here to tell the cooker to make sure to cook these assets
	// when we use a WebBrowserWidget, otherwise the 3D/indirect rendered web browser will not display.
	DefaultMaterial = IWebBrowserModule::Get().GetSingleton()->GetDefaultMaterial();
	DefaultTranslucentMaterial = IWebBrowserModule::Get().GetSingleton()->GetDefaultTranslucentMaterial();
}

void UWebBrowser::LoadURL(FString NewURL)
{
	if ( WebBrowserWidget.IsValid() )
	{
		return WebBrowserWidget->LoadURL(NewURL);
	}
}

void UWebBrowser::LoadString(FString Contents, FString DummyURL)
{
	if ( WebBrowserWidget.IsValid() )
	{
		return WebBrowserWidget->LoadString(Contents, DummyURL);
	}
}

void UWebBrowser::ExecuteJavascript(const FString& ScriptText)
{
	if (WebBrowserWidget.IsValid())
	{
		return WebBrowserWidget->ExecuteJavascript(ScriptText);
	}
}

FText UWebBrowser::GetTitleText() const
{
	if ( WebBrowserWidget.IsValid() )
	{
		return WebBrowserWidget->GetTitleText();
	}

	return FText::GetEmpty();
}

FString UWebBrowser::GetUrl() const
{
	if (WebBrowserWidget.IsValid())
	{
		return WebBrowserWidget->GetUrl();
	}

	return FString();
}

void UWebBrowser::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	WebBrowserWidget.Reset();
}

TSharedRef<SWidget> UWebBrowser::RebuildWidget()
{
	if ( IsDesignTime() )
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Web Browser", "Web Browser"))
			];
	}
	else
	{
		WebBrowserWidget = SNew(SWebBrowser)
			.InitialURL(InitialURL)
			.ShowControls(false)
			.SupportsTransparency(bSupportsTransparency)
			.OnUrlChanged(BIND_UOBJECT_DELEGATE(FOnTextChanged, HandleOnUrlChanged))
			.OnBeforePopup(BIND_UOBJECT_DELEGATE(FOnBeforePopupDelegate, HandleOnBeforePopup))
			.OnConsoleMessage(BIND_UOBJECT_DELEGATE(FOnConsoleMessageDelegate, HandleOnConsoleMessage));

		return WebBrowserWidget.ToSharedRef();
	}
}

void UWebBrowser::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if ( WebBrowserWidget.IsValid() )
	{

	}
}

void UWebBrowser::HandleOnUrlChanged(const FText& InText)
{
	OnUrlChanged.Broadcast(InText);
}

void UWebBrowser::HandleOnConsoleMessage(const FString& Message, const FString& Source, int32 Line, EWebBrowserConsoleLogSeverity Severity)
{
	OnConsoleMessage.Broadcast(Message, Source, Line);
}

bool UWebBrowser::HandleOnBeforePopup(FString URL, FString Frame)
{
	if (OnBeforePopup.IsBound())
	{
		if (IsInGameThread())
		{
			OnBeforePopup.Broadcast(URL, Frame);
		}
		else
		{
			// Retry on the GameThread.
			TWeakObjectPtr<UWebBrowser> WeakThis = this;
			FFunctionGraphTask::CreateAndDispatchWhenReady([WeakThis, URL, Frame]()
			{
				if (WeakThis.IsValid())
				{
					WeakThis->HandleOnBeforePopup(URL, Frame);
				}
			}, TStatId(), nullptr, ENamedThreads::GameThread);
		}

		return true;
	}

	return false;
}

#if WITH_EDITOR

const FText UWebBrowser::GetPaletteCategory()
{
	return LOCTEXT("Experimental", "Experimental");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
