// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserAssetManager.h"

#if WITH_EDITOR || PLATFORM_ANDROID || PLATFORM_IOS
#include "Materials/Material.h"
#include "WebBrowserTexture.h"
#endif

/////////////////////////////////////////////////////
// WebBrowserAssetManager

UWebBrowserAssetManager::UWebBrowserAssetManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) ,
	DefaultMaterial(FSoftObjectPath(FString(TEXT("/Engine/WebBrowser/WebTexture_M.WebTexture_M")))),
	DefaultTranslucentMaterial(FSoftObjectPath(FString(TEXT("/Engine/WebBrowser/WebTexture_TM.WebTexture_TM"))))
{
#if WITH_EDITOR || PLATFORM_ANDROID || PLATFORM_IOS
	// Add a hard reference to UWebBrowserTexture, without this the WebBrowserTexture DLL never gets loaded on Windows.
	UWebBrowserTexture::StaticClass();

#endif
};

void UWebBrowserAssetManager::LoadDefaultMaterials()
{
	DefaultMaterial.LoadSynchronous();
	DefaultTranslucentMaterial.LoadSynchronous();
}

UMaterial* UWebBrowserAssetManager::GetDefaultMaterial()
{
	return DefaultMaterial.Get();
}

UMaterial* UWebBrowserAssetManager::GetDefaultTranslucentMaterial()
{
	return DefaultTranslucentMaterial.Get();
}
