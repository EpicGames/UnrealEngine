// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserWidgetModule.h"
#include "WebBrowserAssetManager.h"
#include "WebBrowserModule.h"
#include "IWebBrowserSingleton.h"
#include "Materials/Material.h"

//////////////////////////////////////////////////////////////////////////
// FWebBrowserWidgetModule

class FWebBrowserWidgetModule : public IWebBrowserWidgetModule
{
public:
	virtual void StartupModule() override
	{
		IWebBrowserModule::Get(); // force the module to load
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FWebBrowserWidgetModule, WebBrowserWidget);
