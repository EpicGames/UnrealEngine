// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"

#include "WebBrowser.generated.h"

enum class EWebBrowserConsoleLogSeverity;

/**
 * 
 */
UCLASS()
class WEBBROWSERWIDGET_API UWebBrowser : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnUrlChanged, const FText&, Text);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnBeforePopup, FString, URL, FString, Frame);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnConsoleMessage, const FString&, Message, const FString&, Source, int32, Line);

	/**
	 * Load the specified URL
	 *
	 * @param NewURL New URL to load
	 */
	UFUNCTION(BlueprintCallable, Category="Web Browser")
	void LoadURL(FString NewURL);

	/**
	 * Load a string as data to create a web page
	 *
	 * @param Contents String to load
	 * @param DummyURL Dummy URL for the page
	 */
	UFUNCTION(BlueprintCallable, Category="Web Browser")
	void LoadString(FString Contents, FString DummyURL);

	/**
	* Executes a JavaScript string in the context of the web page
	*
	* @param ScriptText JavaScript string to execute
	*/
	UFUNCTION(BlueprintCallable, Category = "Web Browser")
	void ExecuteJavascript(const FString& ScriptText);

	/**
	 * Get the current title of the web page
	 */
	UFUNCTION(BlueprintCallable, Category="Web Browser")
	FText GetTitleText() const;

	/**
	* Gets the currently loaded URL.
	*
	* @return The URL, or empty string if no document is loaded.
	*/
	UFUNCTION(BlueprintCallable, Category = "Web Browser")
	FString GetUrl() const;

	/** Called when the Url changes. */
	UPROPERTY(BlueprintAssignable, Category = "Web Browser|Event")
	FOnUrlChanged OnUrlChanged;

	/** Called when a popup is about to spawn. */
	UPROPERTY(BlueprintAssignable, Category = "Web Browser|Event")
	FOnBeforePopup OnBeforePopup;

	/** Called when the browser has console spew to print */
	UPROPERTY(BlueprintAssignable, Category = "Web Browser|Event")
	FOnConsoleMessage OnConsoleMessage;

public:

	//~ Begin UWidget interface
	virtual void SynchronizeProperties() override;
	// End UWidget interface

	virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** URL that the browser will initially navigate to. The URL should include the protocol, eg http:// */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FString InitialURL;

	/** Should the browser window support transparency. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	bool bSupportsTransparency;

protected:
	TSharedPtr<class SWebBrowser> WebBrowserWidget;
	
	UMaterialInterface* DefaultMaterial;
	UMaterialInterface* DefaultTranslucentMaterial;

protected:
	// UWidget interface
	virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	void HandleOnConsoleMessage(const FString& Message, const FString& Source, int32 Line, EWebBrowserConsoleLogSeverity Severity);
	void HandleOnUrlChanged(const FText& Text);
	bool HandleOnBeforePopup(FString URL, FString Frame);
};
