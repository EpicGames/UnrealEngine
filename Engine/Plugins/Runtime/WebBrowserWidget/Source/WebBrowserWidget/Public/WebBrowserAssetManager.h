// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"
#include "Materials/Material.h"
#include "WebBrowserAssetManager.generated.h"

class UMaterial;
/**
 * 
 */

UCLASS()
class WEBBROWSERWIDGET_API UWebBrowserAssetManager : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.8, "UWebBrowserAssetManager is deprecated. LoadDefaultMaterials is no longer necessary for the GetDefaultMaterial functions in IWebBrowserSingleton.")
	void LoadDefaultMaterials();

	UE_DEPRECATED(5.8, "UWebBrowserAssetManager is deprecated. Please use the GetDefaultMaterial functions in IWebBrowserSingleton instead.")
	UMaterial* GetDefaultMaterial(); 
	UE_DEPRECATED(5.8, "UWebBrowserAssetManager is deprecated. Please use the GetDefaultMaterial functions in IWebBrowserSingleton instead.")
	UMaterial* GetDefaultTranslucentMaterial(); 

protected:
	UPROPERTY()
	TSoftObjectPtr<UMaterial> DefaultMaterial;
	TSoftObjectPtr<UMaterial> DefaultTranslucentMaterial;
};
