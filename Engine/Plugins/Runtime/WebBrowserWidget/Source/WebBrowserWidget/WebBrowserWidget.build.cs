// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class WebBrowserWidget : ModuleRules
    {
        public WebBrowserWidget(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "WebBrowser",
                    "Slate",
                    "SlateCore",
                    "UMG",
                    "Engine"
                }
            );

            if (Target.bBuildEditor == true)
            {
                //@TODO: UnrealEd Needed for the triangulation code used for sprites (but only in editor mode)
                //@TOOD: Try to move the code dependent on the triangulation code to the editor-only module
                PrivateIncludePathModuleNames.AddRange(
                    new string[]
                    {
                        "UnrealEd",
                    }
                );
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
						"EditorFramework",
                        "UnrealEd",
                    }
                );
            }
			
			if (Target.bBuildEditor || Target.Platform == UnrealTargetPlatform.Android || Target.Platform == UnrealTargetPlatform.IOS)
			{
                // WebBrowserTexture required for cooking Android
                PrivateIncludePathModuleNames.AddRange(
                    new string[]
                    {
                        "WebBrowserTexture",
                    }
                );
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
                        "WebBrowserTexture",
                    }
                );

				if (Target.Platform == UnrealTargetPlatform.Android)
				{
					string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
					AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "WebBrowserWidget_UPL.xml"));
				}
			}
        }
    }
}