<?xml version="1.0" encoding="utf-8"?>
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<init>
		<log text="WebBrowserWidget Plugin Init"/>

		<setBoolFromProperty result="bPackageForMetaQuest" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bPackageForMetaQuest" default="false"/>

		<if condition="bPackageForMetaQuest">
			<true>
				<log text="Adding com.oculus.always_draw_view_root for Oculus devices to manifest for hardware acceleration."/>
			</true>
		</if>
	</init>

	<androidManifestUpdates>
		<if condition="bPackageForMetaQuest">
			<true>
				<addElements tag="application">
					<meta-data android:name="com.oculus.always_draw_view_root" android:value="true"/>
				</addElements>
			</true>
		</if>
	</androidManifestUpdates>

</root>
