{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Web Browser",
	"Description": "Allows the user to create a Web Browser Widget",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebBrowserWidget",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux",
				"Android",
				"IOS"
			]
		}
	],
	"IsExperimentalVersion": false
}